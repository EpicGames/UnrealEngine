// Copyright Epic Games, Inc. All Rights Reserved.


half3 LinearTo709Branchless(half3 lin) 
{
	return min(lin * 4.5, pow(max(lin, 0.018), 0.45) * 1.099 - 0.099);
}

half3 LinearToSrgbBranchless(half3 lin) 
{
	return min(lin * 12.92, pow(max(lin, 0.00313067), 1.0/2.4) * 1.055 - 0.055);
	// Possible that mobile GPUs might have native pow() function?
	//return min(lin * 12.92, exp2(log2(max(lin, 0.00313067)) * (1.0/2.4) + log2(1.055)) - 0.055);
}

half LinearToSrgbBranchingChannel(half lin) 
{
	if(lin < 0.00313067) return lin * 12.92;
	return pow(lin, (1.0/2.4)) * 1.055 - 0.055;
}

half3 LinearToSrgbBranching(half3 lin) 
{
	return half3(
		LinearToSrgbBranchingChannel(lin.r),
		LinearToSrgbBranchingChannel(lin.g),
		LinearToSrgbBranchingChannel(lin.b));
}

half3 sRGBToLinear( half3 Color ) 
{
	return Color > 0.04045 ? pow( Color * (1.0 / 1.055) + 0.0521327, 2.4 ) : Color * (1.0 / 12.92);
}

/**
 * @param GammaCurveRatio The curve ratio compared to a 2.2 standard gamma, e.g. 2.2 / DisplayGamma.  So normally the value is 1.
 */
half3 ApplyGammaCorrection(float3 LinearColor, float GammaCurveRatio)
{
	// Apply "gamma" curve adjustment.
	float3 CorrectedColor = pow(LinearColor, GammaCurveRatio);

	#if MAC
		// Note, MacOSX native output is raw gamma 2.2 not sRGB!
		CorrectedColor = pow(CorrectedColor, 1.0/2.2);
	#else
		#if USE_709
			// Didn't profile yet if the branching version would be faster (different linear segment).
			CorrectedColor = LinearTo709Branchless(CorrectedColor);
		#else
			CorrectedColor = LinearToSrgbBranching(CorrectedColor);
		#endif
	#endif

	return CorrectedColor;
}