// Copyright Epic Games, Inc. All Rights Reserved.

#include "GammaCorrectionCommon.hlsl"

cbuffer PerElementVSConstants
{
	matrix WorldViewProjection;
}

struct VertexOut
{
	float4 Position : SV_POSITION;
	float4 Color : COLOR0;
	float4 SecondaryColor : COLOR1;
	float4 TextureCoordinates : TEXCOORD0;
};

VertexOut Main(
	in float2 InPosition : POSITION,
	in float4 InTextureCoordinates : TEXCOORD0,
	in float2 MaterialTexCoords : TEXCOORD1,
	in float4 InColor : COLOR0,
	in float4 InSecondaryColor : COLOR1
	)
{
	VertexOut Out;

	Out.Position = mul( WorldViewProjection, float4( InPosition.xy, 0, 1 ) );

	Out.TextureCoordinates = InTextureCoordinates;
	
	InColor.rgb = sRGBToLinear(InColor.rgb);
	InSecondaryColor.rgb = sRGBToLinear(InSecondaryColor.rgb);

	Out.Color = InColor;
	Out.SecondaryColor = InSecondaryColor;

	return Out;
}
