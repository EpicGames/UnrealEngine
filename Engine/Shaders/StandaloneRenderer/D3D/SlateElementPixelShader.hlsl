// Copyright Epic Games, Inc. All Rights Reserved.

#include "GammaCorrectionCommon.hlsl"

// Shader types
#define ESlateShader::Default		0
#define ESlateShader::Border		1
#define ESlateShader::GrayscaleFont	2
#define ESlateShader::ColorFont		3
#define ESlateShader::LineSegment	4
#define ESlateShader::Custom		5
#define ESlateShader::RoundedBox    7

#define USE_LEGACY_DISABLED_EFFECT 0

Texture2D ElementTexture;
SamplerState ElementTextureSampler;

cbuffer PerFramePSConstants
{
	/** Display gamma x:gamma curve adjustment, y:inverse gamma (1/GEngine->DisplayGamma) */
	float2 GammaValues;
};

cbuffer PerElementPSConstants
{
    float4 ShaderParams;        // 16 bytes
	float4 ShaderParams2;		// 16 bytes
	uint ShaderType;            //  4 bytes
    uint IgnoreTextureAlpha;    //	4 bytes
    uint DisableEffect;         //  4 bytes
    uint UNUSED[1];             //  4 bytes
};

struct VertexOut
{
	float4 Position : SV_POSITION;
	float4 Color : COLOR0;
	float4 SecondaryColor : COLOR1;
	float4 TextureCoordinates : TEXCOORD0;
};

float3 Hue( float H )
{
	float R = abs(H * 6 - 3) - 1;
	float G = 2 - abs(H * 6 - 2);
	float B = 2 - abs(H * 6 - 4);
	return saturate( float3(R,G,B) );
}

float3 GammaCorrect(float3 InColor)
{
	float3 CorrectedColor = InColor;

	if ( GammaValues.y != 1.0f )
	{
		CorrectedColor = ApplyGammaCorrection(CorrectedColor, GammaValues.x);
	}

	return CorrectedColor;
}

float4 GetGrayscaleFontElementColor( VertexOut InVertex )
{
	float4 OutColor = InVertex.Color;

	OutColor.a *= ElementTexture.Sample(ElementTextureSampler, InVertex.TextureCoordinates.xy).a;
	
	return OutColor;
}

float4 GetColorFontElementColor(VertexOut InVertex)
{
	float4 OutColor = InVertex.Color;

	OutColor *= ElementTexture.Sample(ElementTextureSampler, InVertex.TextureCoordinates.xy);

	return OutColor;
}

float4 GetColor( VertexOut InVertex, float2 UV )
{
	float4 FinalColor;
	
	float4 BaseColor = ElementTexture.Sample(ElementTextureSampler, UV );
	if( IgnoreTextureAlpha != 0 )
	{
		BaseColor.a = 1.0f;
	}

	FinalColor = BaseColor*InVertex.Color;
	return FinalColor;
}

float4 GetDefaultElementColor( VertexOut InVertex )
{
	return GetColor( InVertex, InVertex.TextureCoordinates.xy*InVertex.TextureCoordinates.zw );
}

float4 GetBorderElementColor( VertexOut InVertex )
{
	float2 NewUV;
	if( InVertex.TextureCoordinates.z == 0.0f && InVertex.TextureCoordinates.w == 0.0f )
	{
		NewUV = InVertex.TextureCoordinates.xy;
	}
	else
	{
		float2 MinUV;
		float2 MaxUV;
	
		if( InVertex.TextureCoordinates.z > 0 )
		{
			MinUV = float2(ShaderParams.x,0);
			MaxUV = float2(ShaderParams.y,1);
			InVertex.TextureCoordinates.w = 1.0f;
		}
		else
		{
			MinUV = float2(0,ShaderParams.z);
			MaxUV = float2(1,ShaderParams.w);
			InVertex.TextureCoordinates.z = 1.0f;
		}

		NewUV = InVertex.TextureCoordinates.xy*InVertex.TextureCoordinates.zw;
		NewUV = frac(NewUV);
		NewUV = lerp(MinUV,MaxUV,NewUV);	
	}

	return GetColor( InVertex, NewUV );
}

float GetRoundedBoxDistance(float2 pos, float2 center, float radius, float inset)
{
	// distance from center
    pos = abs(pos - center); 

    // distance from the inner corner
    pos = pos - (center - float2(radius + inset, radius + inset));

    // use distance to nearest edge when not in quadrant with radius
    // this handles an edge case when radius is very close to thickness
    // otherwise we're in the quadrant with the radius, 
    // just use the analytic signed distance function
    return lerp( length(pos) - radius, max(pos.x - radius, pos.y - radius), float(pos.x <= 0 || pos.y <=0) );
}

float4 GetRoundedBoxElementColor( VertexOut InVertex )
{
	const float2 size = ShaderParams.zw;
	float2 pos = size * InVertex.TextureCoordinates.xy;
	float2 center = size / 2.0;

	//X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left */
	float4 cornerRadii = ShaderParams2;

	// figure out which radius to use based on which quadrant we're in
	float2 quadrant = step(InVertex.TextureCoordinates.xy, float2(.5,.5));

	float left = lerp(cornerRadii.y, cornerRadii.x, quadrant.x);
	float right = lerp(cornerRadii.z, cornerRadii.w, quadrant.x);
	float radius = lerp(right, left, quadrant.y);

	float thickness = ShaderParams.y; 

	// Compute the distances internal and external to the border outline
	float dext = GetRoundedBoxDistance(pos, center, radius, 0.0);
	float din  = GetRoundedBoxDistance(pos, center, max(radius - thickness, 0), thickness);

	// Compute the border intensity and fill intensity with a smooth transition
	float spread = 0.5;
	float bi = smoothstep(spread, -spread, dext);
	float fi = smoothstep(spread, -spread, din);

	// alpha blend the external color 
	float4 fill = GetColor(InVertex, InVertex.TextureCoordinates.xy * InVertex.TextureCoordinates.zw);
	float4 border = InVertex.SecondaryColor;
	float4 OutColor = lerp(border, fill, float(thickness > radius));
	OutColor.a = 0.0;

	// blend in the border and fill colors
	OutColor = lerp(OutColor, border, bi);
	OutColor = lerp(OutColor, fill, fi);
	return OutColor;
}

/**
 * Computes 0-1 value that interpolates between dashes of length DashLength based on Distance with 1px of antialiasing between dashes and gaps
 * 
 * ie:
 * 1 -----------------                     -----------------                     -----------------                     
 *                    \                   /                 \                   /                 \                   /
 * 0                   \_________________/                   \_________________/                   \_________________/ 
 *
 *   <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >
 
 */
float ModulateDashedLine(float Distance, float DashLength)
{
	// Add one for antialias pixel
	DashLength += 1.f;

	const float HalfDash  = 0.5f*DashLength;
	const float Period    = 2.0f*DashLength;

	// Offset the interpolation so that the dash starts at distance = 0;
	const float Interp = Distance - HalfDash + 0.5f;
	const float Mod = Interp - Period*floor(Interp/Period); // mod(Interp, Period)

	// Compute the dash alpha using a clamped triangle wave
	const float TriWave = 2.0f * abs(Mod - DashLength) - DashLength;
	return clamp(TriWave, -1.0f, 1.0f)*0.5f + 0.5f;
}

/**
 * Generates an anti-aliased line segment pixel
 * This is based on the fast prefiltered lines technique published in GPU Gems 2
 *
 * Instead of passing edge function coefficients as uniform parameters we use
 * texture coordinates to define the line
 *
 * When rasterized, the submitted geometry must contain every pixel that
 * could have coverage > 0. Failing to do so will create sharp, aliased edges
 */
float4 GetLineSegmentElementColor( VertexOut InVertex )
{
	const float2 Gradient = InVertex.TextureCoordinates;
	const float2 DashParams = InVertex.TextureCoordinates.zw;

	// Get coverage based on distance from segment sides and ends
	const float2 OutsideFilterUV = float2(1.0f, 1.0f);
	const float2 InsideFilterUV = float2(ShaderParams.x, 0.0f);
	const float2 LineCoverage = smoothstep(OutsideFilterUV, InsideFilterUV, abs(Gradient));

	const float DashLength = DashParams.y;
	const float DashAlpha = ModulateDashedLine(DashParams.x, DashParams.y);

	float4 Color = InVertex.Color;
	Color.a *= LineCoverage.x * LineCoverage.y * DashAlpha;
	return Color;
}

float4 Main( VertexOut InVertex ) : SV_Target
{
	float4 OutColor;

	if( ShaderType == ESlateShader::Default || ShaderType == ESlateShader::Custom )
	{
		OutColor = GetDefaultElementColor( InVertex );
	}
	else if( ShaderType == ESlateShader::RoundedBox)
	{
		OutColor = GetRoundedBoxElementColor( InVertex );
	}
	else if( ShaderType == ESlateShader::Border )
	{
		OutColor = GetBorderElementColor( InVertex );
	}
	else if( ShaderType == ESlateShader::GrayscaleFont )
	{
		OutColor = GetGrayscaleFontElementColor( InVertex );
	}
	else if (ShaderType == ESlateShader::ColorFont)
	{
		OutColor = GetColorFontElementColor(InVertex);
	}
	else
	{
		OutColor = GetLineSegmentElementColor( InVertex );
	}

	// gamma correct
	OutColor.rgb = GammaCorrect(OutColor.rgb);

    if (DisableEffect)
	{
#if USE_LEGACY_DISABLED_EFFECT

		//desaturate
		float3 LumCoeffs = float3( 0.3, 0.59, .11 );
		float Lum = dot( LumCoeffs, OutColor.rgb );
		OutColor.rgb = lerp( OutColor.rgb, float3(Lum,Lum,Lum), .8 );
	
		float3 Grayish = {.4, .4, .4};
		
		// lerp between desaturated color and gray color based on distance from the desaturated color to the gray
		OutColor.rgb = lerp( OutColor.rgb, Grayish, clamp( distance( OutColor.rgb, Grayish ), 0, .8)  );
#else
		OutColor.a *= .45f;
#endif
	}

	return OutColor;
}

