// Copyright Epic Games, Inc. All Rights Reserved.

// handle differences between ES and full GL shaders
#if PLATFORM_USES_GLES
precision highp float;
#else
// #version 120 at the beginning is added in FSlateOpenGLShader::CompileShader()
#extension GL_EXT_gpu_shader4 : enable
#endif

#ifndef USE_709
#define USE_709 0
#endif // USE_709

// Shader types
#define ST_Default			0
#define ST_Border			1
#define ST_GrayscaleFont	2
#define ST_ColorFont		3
#define ST_Line				4
#define ST_Custom			5
#define ST_RoundedBox       7

#define USE_LEGACY_DISABLED_EFFECT 0

/** Display gamma x:gamma curve adjustment, y:inverse gamma (1/GEngine->DisplayGamma) */
uniform vec2 GammaValues = vec2(1, 1/2.2);

// Draw effects
uniform bool EffectsDisabled;
uniform bool IgnoreTextureAlpha;

uniform vec4 ShaderParams;
uniform vec4 ShaderParams2;
uniform int ShaderType;
uniform sampler2D ElementTexture;

#if PLATFORM_MAC
// GL_TEXTURE_RECTANGLE_ARB support, used by the web surface on macOS
uniform bool UseTextureRectangle;
uniform sampler2DRect ElementRectTexture;
uniform vec2 Size;
#endif

varying vec4 Position;
varying vec4 TexCoords;
varying vec4 Color;
varying vec4 SecondaryColor;

vec3 maxWithScalar(float test, vec3 values)
{
	return vec3(max(test, values.x), max(test, values.y), max(test, values.z));
}

vec3 powScalar(vec3 values, float power)
{
	return vec3(pow(values.x, power), pow(values.y, power), pow(values.z, power));
}

vec3 LinearTo709Branchless(vec3 lin)
{
	return min(lin * 4.5, powScalar(maxWithScalar(0.018, lin), 0.45) * 1.099 - 0.099);
}

vec3 LinearToSrgbBranchless(vec3 lin)
{
	return min(lin * 12.92, powScalar(maxWithScalar(0.00313067, lin), 1.0/2.4) * 1.055 - 0.055);
	// Possible that mobile GPUs might have native pow() function?
	//return min(lin * 12.92, exp2(log2(max(lin, 0.00313067)) * (1.0/2.4) + log2(1.055)) - 0.055);
}

float LinearToSrgbBranchingChannel(float lin)
{
	if(lin < 0.00313067) return lin * 12.92;
	return pow(lin, (1.0/2.4)) * 1.055 - 0.055;
}

vec3 LinearToSrgbBranching(vec3 lin)
{
	return vec3(
				LinearToSrgbBranchingChannel(lin.r),
				LinearToSrgbBranchingChannel(lin.g),
				LinearToSrgbBranchingChannel(lin.b));
}

float sRGBToLinearChannel( float ColorChannel )
{
	return ColorChannel > 0.04045 ? pow( ColorChannel * (1.0 / 1.055) + 0.0521327, 2.4 ) : ColorChannel * (1.0 / 12.92);
}

vec3 sRGBToLinear( vec3 Color )
{
	return vec3(sRGBToLinearChannel(Color.r),
				sRGBToLinearChannel(Color.g),
				sRGBToLinearChannel(Color.b));
}

/**
 * @param GammaCurveRatio The curve ratio compared to a 2.2 standard gamma, e.g. 2.2 / DisplayGamma.  So normally the value is 1.
 */
vec3 ApplyGammaCorrection(vec3 LinearColor, float GammaCurveRatio)
{
	// Apply "gamma" curve adjustment.
	vec3 CorrectedColor = powScalar(LinearColor, GammaCurveRatio);

#if PLATFORM_MAC
	// Note, MacOSX native output is raw gamma 2.2 not sRGB!
	//CorrectedColor = pow(CorrectedColor, 1.0/2.2);
	CorrectedColor = LinearToSrgbBranching(CorrectedColor);
#else
#if USE_709
	// Didn't profile yet if the branching version would be faster (different linear segment).
	CorrectedColor = LinearTo709Branchless(CorrectedColor);
#else
	CorrectedColor = LinearToSrgbBranching(CorrectedColor);
#endif
	
#endif
	
	return CorrectedColor;
}

vec3 GammaCorrect(vec3 InColor)
{
	vec3 CorrectedColor = InColor;
	
	// gamma correct
	//#if PLATFORM_USES_GLES
	//	OutColor.rgb = sqrt( OutColor.rgb );
	//#else
	//	OutColor.rgb = pow(OutColor.rgb, vec3(1.0/2.2));
	//#endif
	
#if !PLATFORM_USES_GLES
	if( GammaValues.y != 1.0f )
	{
		CorrectedColor = ApplyGammaCorrection(CorrectedColor, GammaValues.x);
	}
#endif
	
	return CorrectedColor;
}

vec4 GetGrayscaleFontElementColor()
{
	vec4 OutColor = Color;
#if PLATFORM_LINUX
	OutColor.a *= texture2D(ElementTexture, TexCoords.xy).r; // OpenGL 3.2+ uses Red for single channel textures
#else
	OutColor.a *= texture2D(ElementTexture, TexCoords.xy).a;
#endif

	return OutColor;
}

vec4 GetColorFontElementColor()
{
	vec4 OutColor = Color;

	OutColor *= texture2D(ElementTexture, TexCoords.xy);

	return OutColor;
}

vec4 GetDefaultElementColor()
{
	vec4 OutColor = Color;

	vec4 TextureColor;
#if PLATFORM_MAC
	if ( UseTextureRectangle )
	{
		TextureColor = texture2DRect(ElementRectTexture, TexCoords.xy*TexCoords.zw*Size).bgra;
	}
	else
#endif
	{
		TextureColor = texture2D(ElementTexture, TexCoords.xy*TexCoords.zw);
	}
	
	if( IgnoreTextureAlpha )
	{
		TextureColor.a = 1.0;
	}
	OutColor *= TextureColor;
	return OutColor;
}

vec4 GetBorderElementColor()
{
	vec4 OutColor = Color;
	vec4 InTexCoords = TexCoords;
	vec2 NewUV;
	if( InTexCoords.z == 0.0 && InTexCoords.w == 0.0 )
	{
		NewUV = InTexCoords.xy;
	}
	else
	{
		vec2 MinUV;
		vec2 MaxUV;
	
		if( InTexCoords.z > 0.0 )
		{
			MinUV = vec2(ShaderParams.x,0.0);
			MaxUV = vec2(ShaderParams.y,1.0);
			InTexCoords.w = 1.0;
		}
		else
		{
			MinUV = vec2(0.0,ShaderParams.z);
			MaxUV = vec2(1.0,ShaderParams.w);
			InTexCoords.z = 1.0;
		}

		NewUV = InTexCoords.xy*InTexCoords.zw;
		NewUV = fract(NewUV);
		NewUV = mix(MinUV,MaxUV,NewUV);	

	}

	vec4 TextureColor = texture2D(ElementTexture, NewUV);
	if( IgnoreTextureAlpha )
	{
		TextureColor.a = 1.0;
	}
		
	OutColor *= TextureColor;
	return OutColor;
}

float GetRoundedBoxDistance(vec2 pos, vec2 center, float radius, float inset)
{
	// distance from center
    pos = abs(pos - center); 

    // distance from the inner corner
    pos = pos - (center - vec2(radius + inset, radius + inset));

    // use distance to nearest edge when not in quadrant with radius
    // this handles an edge case when radius is very close to thickness
    // otherwise we're in the quadrant with the radius, 
    // just use the analytic signed distance function
    return mix( length(pos) - radius,
    			max(pos.x - radius, pos.y - radius), 
    			float(pos.x <= 0 || pos.y <=0) );
}

vec4 GetRoundedBoxElementColor()
{
	vec2 size = ShaderParams.zw;
	vec2 pos = size * TexCoords.xy;
	vec2 center = size / 2.0;

	//X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left */
	vec4 cornerRadii = ShaderParams2;

	// figure out which radius to use based on which quadrant we're in
	vec2 quadrant = step(TexCoords.xy, vec2(.5,.5));

	float left = mix(cornerRadii.y, cornerRadii.x, quadrant.x);
	float right = mix(cornerRadii.z, cornerRadii.w, quadrant.x);
	float radius = mix(right, left, quadrant.y);

	float thickness = ShaderParams.y; 

	// Compute the distances internal and external to the border outline
	float dext = GetRoundedBoxDistance(pos, center, radius, 0.0);
	float din  = GetRoundedBoxDistance(pos, center, max(radius - thickness, 0), thickness);

	// Compute the border intensity and fill intensity with a smooth transition
	float spread = 0.5;
	float bi = smoothstep(spread, -spread, dext);
	float fi = smoothstep(spread, -spread, din);

	// alpha blend the external color 
	vec4 fill = GetDefaultElementColor();
	vec4 border = SecondaryColor;
	vec4 OutColor = mix(border, fill, float(thickness > radius));
	OutColor.a = 0.0;

	// blend in the border and fill colors
	OutColor = mix(OutColor, border, bi);
	OutColor = mix(OutColor, fill, fi);
	return OutColor;
}


/**
 * Computes 0-1 value that interpolates between dashes of length DashLength based on Distance with 1px of antialiasing between dashes and gaps
 * 
 * ie:
 * 1 -----------------                     -----------------                     -----------------                     
 *                    \                   /                 \                   /                 \                   /
 * 0                   \_________________/                   \_________________/                   \_________________/ 
 *
 *   <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >  <  DashLength   >
 
 */
float ModulateDashedLine(float Distance, float DashLength)
{
	// Add one for antialias pixel
	DashLength += 1.0;

	float HalfDash  = 0.50*DashLength;
	float Period    = 2.00*DashLength;

	// Offset the interpolation so that the dash starts at distance = 0;
	float Interp = Distance - HalfDash + 0.50;
	float Mod = Interp - Period*floor(Interp/Period); // mod(Interp, Period)

	// Compute the dash alpha using a clamped triangle wave
	float TriWave = 2.0 * abs(Mod - DashLength) - DashLength;
	return clamp(TriWave, -1.0, 1.0)*0.5 + 0.5;
}

/**
 * Generates an anti-aliased line segment pixel
 * This is based on the fast prefiltered lines technique published in GPU Gems 2
 *
 * Instead of passing edge function coefficients as uniform parameters we use
 * texture coordinates to define the line
 *
 * When rasterized, the submitted geometry must contain every pixel that
 * could have coverage > 0. Failing to do so will create sharp, aliased edges
 */
vec4 GetLineSegmentElementColor()
{
	vec2 Gradient = TexCoords.xy;
	vec2 DashParams = TexCoords.zw;

	// Get coverage based on distance from segment sides and ends
	vec2 OutsideFilterUV = vec2(1.0, 1.0);
	vec2 InsideFilterUV = vec2(ShaderParams.x, 0.0);
	vec2 LineCoverage = smoothstep(OutsideFilterUV, InsideFilterUV, abs(Gradient));

	float DashLength = DashParams.y;
	float DashAlpha = ModulateDashedLine(DashParams.x, DashParams.y);

	vec4 OutColor = Color;
	OutColor.a *= LineCoverage.x * LineCoverage.y * DashAlpha;
	return OutColor;
}

void main()
{
	vec4 OutColor;

	if( ShaderType == ST_Default || ShaderType == ST_Custom )
	{
		OutColor = GetDefaultElementColor();
	}
	else if( ShaderType == ST_RoundedBox )
	{
		OutColor = GetRoundedBoxElementColor();
	}
	else if( ShaderType == ST_Border )
	{
		OutColor = GetBorderElementColor();
	}
	else if( ShaderType == ST_GrayscaleFont )
	{
		OutColor = GetGrayscaleFontElementColor();
	}
	else if( ShaderType == ST_ColorFont )
	{
		OutColor = GetColorFontElementColor();
	}
	else
	{
		OutColor = GetLineSegmentElementColor();
	}
	
	// gamma correct
	OutColor.rgb = GammaCorrect(OutColor.rgb);
	
	if( EffectsDisabled )
	{
	#if USE_LEGACY_DISABLED_EFFECT
		//desaturate
		vec3 LumCoeffs = vec3( 0.3, 0.59, .11 );
		float Lum = dot( LumCoeffs, OutColor.rgb );
		OutColor.rgb = mix( OutColor.rgb, vec3(Lum,Lum,Lum), .8 );
	
		vec3 Grayish = vec3(0.4, 0.4, 0.4);
	
		OutColor.rgb = mix( OutColor.rgb, Grayish, clamp( distance( OutColor.rgb, Grayish ), 0.0, 0.8)  );
	#else
		OutColor.a *= .45f;
	#endif
	}

	gl_FragColor = OutColor.bgra;
}
