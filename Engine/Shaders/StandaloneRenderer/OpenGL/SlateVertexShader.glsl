// Copyright Epic Games, Inc. All Rights Reserved.

// #version 120 at the beginning is added in FSlateOpenGLShader::CompileShader()

uniform mat4 ViewProjectionMatrix;

// Per vertex
attribute vec4 InTexCoords;
attribute vec2 InPosition;
attribute vec4 InColor;
attribute vec4 InSecondaryColor;

// Between vertex and pixel shader
varying vec4 Position;
varying vec4 TexCoords;
varying vec4 Color;
varying vec4 SecondaryColor;

vec3 powScalar(vec3 values, float power)
{
	return vec3(pow(values.x, power), pow(values.y, power), pow(values.z, power));
}

float sRGBToLinearChannel( float ColorChannel )
{
	return ColorChannel > 0.04045 ? pow( ColorChannel * (1.0 / 1.055) + 0.0521327, 2.4 ) : ColorChannel * (1.0 / 12.92);
}

vec3 sRGBToLinear( vec3 Color )
{
	return vec3(
				sRGBToLinearChannel(Color.r),
				sRGBToLinearChannel(Color.g),
				sRGBToLinearChannel(Color.b));
}

void main()
{
	TexCoords = InTexCoords;

	Color.rgb = sRGBToLinear(InColor.rgb);
	Color.a = InColor.a;

	SecondaryColor.rgb = sRGBToLinear(InSecondaryColor.rgb);
	SecondaryColor.a = InSecondaryColor.a;

    Position = vec4( InPosition, 0, 1 );
    
	gl_Position = ViewProjectionMatrix * vec4( InPosition, 0, 1 );
}
