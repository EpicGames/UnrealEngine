
#if PLATFORM_USES_GLES
precision highp float;
#else
// #version 120 at the beginning is added in FSlateOpenGLShader::CompileShader()
#extension GL_EXT_gpu_shader4 : enable
#endif

varying vec2 textureCoordinate;

uniform sampler2D SplashTexture;

void main() 
{
	// OpenGL has 0,0 the "math" way
	vec2 tc = vec2(textureCoordinate.s, 1.0-textureCoordinate.t);

	gl_FragColor = texture2D(SplashTexture, tc);

}
