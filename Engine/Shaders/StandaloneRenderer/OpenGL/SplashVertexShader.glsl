attribute vec2 InPosition;

varying vec2 textureCoordinate;

void main() 
{
	// We do not need texture coordinates. We calculate using position.
	textureCoordinate = InPosition * 0.5 + 0.5;

	gl_Position = vec4(InPosition, 0.0, 1.0);

}